{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Avid DNxHR/DNxMXF Media Plugin",
	"Description": "Implements video export using Avid DNx Codecs.",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Category": "Media Players",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AvidDNxMedia",
			"Type": "Runtime",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ],
			"TargetDenyList": [ "Server" ]
		}
	],
	"Plugins":
	[
		{
			"Name": "MovieRenderPipeline",
			"Enabled": true
		}
	]
}
