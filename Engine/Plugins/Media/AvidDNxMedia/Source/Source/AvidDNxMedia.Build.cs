// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AvidDNxMedia : ModuleRules
	{
		public AvidDNxMedia(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"MovieSceneCapture",
					"Projects",
					"MovieRenderPipelineCore",
					"SlateCore",
					"OpenColorIO",
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
                    "DNxHR",
                    "DNxMXF",
                    "DNxUncompressed"
                }
			);
        }
    }
}
