// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvidDNxEncoder/AvidDNxEncoder.h"

#include "Async/ParallelFor.h"
#include "AvidDNxMediaModule.h"
#include "HAL/PlatformTime.h"
#include "Runtime/Launch/Resources/Version.h"

THIRD_PARTY_INCLUDES_START
#include <AvidDNxCodec.h>
#include <dnx_mxf_sdk.h>
THIRD_PARTY_INCLUDES_END


namespace AvidDNx
{
	// identifier for this program which will be embedded in exported mxf files
	const wchar_t* ProductUID = L"06.9d.41.48.a0.cb.48.d4.af.19.54.da.bd.09.2a.9f";

	/**
	 * Converts two RGB pixels (InputColor0 and InputColor1) to sub-sampled YCbCr Rec. 709 and video range. Supports both 8-bit and 16-bit color.
	 *
	 * InputType supports FColor and FFloat16Color.
	 * ColorComponentType supports uint8 and unsigned short.
	 * ColorContainer supports FY0CbY1Cr and FY0CbY1Cr_16bit.
	 */
	template<typename InputType, typename ColorComponentType, typename ColorContainer>
	ColorContainer RBGtoYCbCrRec709(const InputType* InputColor0, const InputType* InputColor1)
	{
		static_assert(std::is_same_v<ColorComponentType, uint8> || std::is_same_v<ColorComponentType, unsigned short>, "Unsupported color component type");

		float R0, G0, B0, R1, G1, B1;

		if constexpr (std::is_same_v<ColorComponentType, uint8>)
		{
			R0 = FMath::Clamp(InputColor0->R / 255.f, 0.f, 1.f);
			G0 = FMath::Clamp(InputColor0->G / 255.f, 0.f, 1.f);
			B0 = FMath::Clamp(InputColor0->B / 255.f, 0.f, 1.f);

			R1 = FMath::Clamp(InputColor1->R / 255.f, 0.f, 1.f);
			G1 = FMath::Clamp(InputColor1->G / 255.f, 0.f, 1.f);
			B1 = FMath::Clamp(InputColor1->B / 255.f, 0.f, 1.f);
		}
		else
		{
			// ColorComponentType == unsigned short
			
			R0 = FMath::Clamp(InputColor0->R.GetFloat(), 0.f, 1.f);
			G0 = FMath::Clamp(InputColor0->G.GetFloat(), 0.f, 1.f);
			B0 = FMath::Clamp(InputColor0->B.GetFloat(), 0.f, 1.f);

			R1 = FMath::Clamp(InputColor1->R.GetFloat(), 0.f, 1.f);
			G1 = FMath::Clamp(InputColor1->G.GetFloat(), 0.f, 1.f);
			B1 = FMath::Clamp(InputColor1->B.GetFloat(), 0.f, 1.f);
		}

		// Rec. 709 conversion
		// See: https://en.wikipedia.org/wiki/YCbCr#ITU-R_BT.709_conversion
		const float Yfull0 = R0 * 0.212639f + G0 * 0.7151687f + B0 * 0.0721932f;
		const float Yfull1 = R1 * 0.212639f + G1 * 0.7151687f + B1 * 0.0721932f;
		const float CbFull0 = R0 * (-0.1145922f) + G0 * (-0.3854078f) + B0 * 0.5f;
		const float CbFull1 = R1 * (-0.1145922f) + G1 * (-0.3854078f) + B1 * 0.5f;
		const float CrFull0 = R0 * 0.5f + G0 * (-0.4541555f) + B0 * (-0.04584448f);
		const float CrFull1 = R1 * 0.5f + G1 * (-0.4541555f) + B1 * (-0.04584448f);

		const float CbAvg = (CbFull0 + CbFull1) / 2.0f;
		const float CrAvg = (CrFull0 + CrFull1) / 2.0f;

		// Video range conversion. 8-bit values are from the Rec. 709 specification. The 16-bit values were derived from this.
		// Example: WhitePoint_Y_16bit = 235/256 * 65536, where 256 is the max value of a uint8, and 65536 for uint16.
		constexpr uint8 WhitePoint_Y_8bit = 235;
		constexpr uint8 WhitePoint_CbCr_8bit = 240;
		constexpr uint8 BlackPoint_8bit = 16;
		constexpr uint8 Midpoint_8bit = 128;
		
		constexpr uint16 WhitePoint_Y_16bit = 60160;
		constexpr uint16 WhitePoint_CbCr_16bit = 61440;
		constexpr uint16 BlackPoint_16bit = 4096;
		constexpr uint16 Midpoint_16bit = 32768;

		ColorComponentType RangeDifference_Y;
		ColorComponentType RangeDifference_CbCr;
		ColorComponentType BlackPoint;
		ColorComponentType Midpoint;
		if constexpr (std::is_same_v<ColorComponentType, uint8>)
		{
			RangeDifference_Y = WhitePoint_Y_8bit - BlackPoint_8bit;
			RangeDifference_CbCr = WhitePoint_CbCr_8bit - BlackPoint_8bit;
			BlackPoint = BlackPoint_8bit;
			Midpoint = Midpoint_8bit;
		}
		else if constexpr (std::is_same_v<ColorComponentType, unsigned short>)
		{
			RangeDifference_Y = WhitePoint_Y_16bit - BlackPoint_16bit;
			RangeDifference_CbCr = WhitePoint_CbCr_16bit - BlackPoint_16bit;
			BlackPoint = BlackPoint_16bit;
			Midpoint = Midpoint_16bit;
		}
		
		const ColorComponentType YVideoRange0 = static_cast<ColorComponentType>(FMath::RoundToInt((RangeDifference_Y * Yfull0 + BlackPoint)));
		const ColorComponentType YVideoRange1 = static_cast<ColorComponentType>(FMath::RoundToInt((RangeDifference_Y * Yfull1 + BlackPoint)));
		const ColorComponentType CbVideoRange = static_cast<ColorComponentType>(FMath::RoundToInt((RangeDifference_CbCr * CbAvg + Midpoint)));
		const ColorComponentType CrVideoRange = static_cast<ColorComponentType>(FMath::RoundToInt((RangeDifference_CbCr * CrAvg + Midpoint)));

		return ColorContainer(YVideoRange0, CbVideoRange, YVideoRange1, CrVideoRange);
	}

	/** Converts a FFloat16Color to Rec. 709 and video range. */
	FAvidDNxEncoder::FRGB_16bit RGBtoRec709(const FFloat16Color* InColor)
	{
		constexpr unsigned short WhitePoint = 60160;
		constexpr unsigned short BlackPoint = 4096;
		constexpr unsigned short Difference = WhitePoint - BlackPoint;
		constexpr float GammaExponent = 1.f / 2.4f;

		// Rec. 709 conversion
		const float R_Clamped = FMath::Clamp(InColor->R, 0.f, 1.f);
		const float G_Clamped = FMath::Clamp(InColor->G, 0.f, 1.f);
		const float B_Clamped = FMath::Clamp(InColor->B, 0.f, 1.f);
		const float R = (R_Clamped <= 0.0031308f) ? R_Clamped * 12.92f : 1.055f * FMath::Pow(R_Clamped, GammaExponent) - 0.055f;
		const float G = (G_Clamped <= 0.0031308f) ? G_Clamped * 12.92f : 1.055f * FMath::Pow(G_Clamped, GammaExponent) - 0.055f;
		const float B = (B_Clamped <= 0.0031308f) ? B_Clamped * 12.92f : 1.055f * FMath::Pow(B_Clamped, GammaExponent) - 0.055f;

		// Convert to video range
		return FAvidDNxEncoder::FRGB_16bit(
			static_cast<unsigned short>(R * Difference + BlackPoint),
			static_cast<unsigned short>(G * Difference + BlackPoint),
			static_cast<unsigned short>(B * Difference + BlackPoint));
	}

	int32 GCD(int32 A, int32 B)
	{
		while (B != 0)
		{
			int32 Temp = B;
			B = A % B;
			A = Temp;
		}

		return A;
	}

	DNXMXF_Rational_t AspectRatioFromResolution(const int32 InWidth, const int32 InHeight)
	{
		const int32 Gcd = GCD(InWidth, InHeight);
		return { (unsigned int)(InWidth / Gcd), (unsigned int)(InHeight / Gcd) };
	}

	void DNxUncompressedErrorHandler(const char* Message, void* /*UserData*/)
	{
		UE_LOGF(LogAvidDNxMedia, Error, "DNxUncompressed Error: %s", Message);
	}

	void MXFerrorHandler(const char* Message, void* /*UserData*/)
	{
		UE_LOGF(LogAvidDNxMedia, Error, "Error initializing DNX MXF SDK: %s", Message);
	}
}

FAvidDNxEncoder::FAvidDNxEncoder(const FAvidDNxEncoderOptions& InOptions)
	: Options(InOptions)
	, bInitialized(false)
	, bFinalized(false)
	, WriteStartTimeSeconds(0)
	, WriteEndTimeSeconds(0)
	, DNxHRencoder(nullptr)
	, DNxUncEncoder(nullptr)
	, MXFwriter(nullptr)
{
}


FAvidDNxEncoder::~FAvidDNxEncoder()
{
	// Insure Finalize is called so that we release resources if we were ever initialized.
	Finalize();
}

bool FAvidDNxEncoder::Initialize()
{
	const int InitResult = DNX_Initialize();
	if (InitResult != DNX_NO_ERROR)
	{
		char ErrorMessage[256];
		DNX_GetErrorString(InitResult, &ErrorMessage[0]);
		UE_LOGF(LogAvidDNxMedia, Error, "Error initializing DNX SDK: %s", ErrorMessage);
		return false;
	}

	bool bSuccess = false;
	if (Options.bCompress)
	{
		bSuccess = InitializeCompressedEncoder();
	}
	else
	{
		bSuccess = InitializeUncompressedEncoder();
	}
	if (!bSuccess)
	{
		UE_LOGF(LogAvidDNxMedia, Error, "Failed to initialize Encoder. Compressed: %d", Options.bCompress);
		return false;
	}

	bSuccess = InitializeMXFWriter();
	if (!bSuccess)
	{
		UE_LOGF(LogAvidDNxMedia, Error, "Failed to initialize output file. File Path: %ls", *Options.OutputFilename);
		return false;
	}

	bInitialized = true;
	return true;
}

bool FAvidDNxEncoder::InitializeCompressedEncoder()
{
	const bool bIsRGB = (Options.Quality == EAvidDNxEncoderQuality::RGB444_12bit);
	
	DNX_ComponentType_t ComponentType = DNX_CT_UCHAR;
	if (Options.Quality == EAvidDNxEncoderQuality::HQX_10bit)
	{
		ComponentType = DNX_CT_USHORT_10_6;
	}
	else if (Options.Quality == EAvidDNxEncoderQuality::RGB444_12bit)
	{
		ComponentType = DNX_CT_USHORT_12_4;
	}
	
	const DNX_UncompressedParams_t UncompressedParamsHR =
	{
		sizeof(DNX_UncompressedParams_t),
		ComponentType, // Component type
		DNX_CV_709, // Color volume
		bIsRGB ? DNX_CF_RGB : DNX_CF_YCbCr, // Color format
		bIsRGB ? DNX_CCO_RGB_NoA : DNX_CCO_YCbYCr_NoA, // Component order
		DNX_BFO_Progressive, // Field order
		DNX_RGT_Display, // Raster geometry type
		0, // Interfield gap bytes
		0, // Row bytes
		// Used only for DNX_CT_SHORT_2_14
		0, // Black point
		0, // White point
		0, // ChromaExcursion
		// Used only for planar component orders
		0 // Row bytes2
	};

	unsigned int BitDepth = 8;
	if (Options.Quality == EAvidDNxEncoderQuality::HQX_10bit)
	{
		BitDepth = 10;
	}
	else if (Options.Quality == EAvidDNxEncoderQuality::RGB444_12bit)
	{
		BitDepth = 12;
	}

	const DNX_CompressedParams_t CompressedParamsHR =
	{
		sizeof(DNX_CompressedParams_t),
		Options.Width,
		Options.Height,
		static_cast<DNX_CompressionID_t>(Options.Quality),
		DNX_CV_709, // Color volume
		bIsRGB ? DNX_CF_RGB : DNX_CF_YCbCr, // Color format
		// Parameters below are used for RI only
		bIsRGB ? DNX_SSC_444 : DNX_SSC_422, // Sub-sampling
		BitDepth, // bit-depth, is used only for RI compression IDs
		1, // PARC
		1, // PARN
		0, // CRC-presence
		0, // VBR
		0, // Alpha-presence
		0, // Lossless alpha
		0  // Premultiplied alpha
	};

	const DNX_EncodeOperationParams_t OperationParams =
	{
		sizeof(DNX_EncodeOperationParams_t),
		FMath::Max<unsigned int>(Options.NumberOfEncodingThreads, 1u)
	};

	const int CreateResult = DNX_CreateEncoder(&CompressedParamsHR, &UncompressedParamsHR, &OperationParams, &DNxHRencoder);
	if (CreateResult != DNX_NO_ERROR)
	{
		char ErrorMessage[256];
		DNX_GetErrorString(CreateResult, &ErrorMessage[0]);
		UE_LOGF(LogAvidDNxMedia, Error, "Error initializing DNX Encoder: %s", ErrorMessage);
		return false;
	}

	EncodedBufferSize = DNX_GetCompressedBufferSize(&CompressedParamsHR);
	return true;
}

bool FAvidDNxEncoder::InitializeUncompressedEncoder()
{
	const DNXUncompressed_Options_t UncompressedOptions =
	{
		sizeof(DNXUncompressed_Options_t),
		Options.NumberOfEncodingThreads,
		nullptr,
		AvidDNx::DNxUncompressedErrorHandler
	};

	if (DNX_UNCOMPRESSED_ERR_SUCCESS != DNXUncompressed_CreateEncoder(&UncompressedOptions, &DNxUncEncoder))
	{
		UE_LOGF(LogAvidDNxMedia, Error, "Failed to initialize Uncompressed Encoder.");
		return false;
	}

	DNxUncUncompressedParams = DNXUncompressed_UncompressedParams_t
	{
			sizeof(DNXUncompressed_UncompressedParams_t),
			DNX_UNCOMPRESSED_CCO_YCbYCr, // color component order
			DNX_UNCOMPRESSED_CT_UCHAR, // component type
			(unsigned int)Options.Width,
			(unsigned int)Options.Height,
			0, // row bytes
			DNX_UNCOMPRESSED_FL_FULL_FRAME, // frame layout
			0 // row bytes 2
	};

	DNxUncCompressedParams = DNXUncompressed_CompressedParams_t
	{
		sizeof(DNXUncompressed_CompressedParams_t),
		false, // compress alpha
		0 // slice count for RLE if compress alpha is enabled
	};

	EncodedBufferSize = DNXUncompressed_GetCompressedBufSize(&DNxUncUncompressedParams, &DNxUncCompressedParams);
	return true;
}

bool FAvidDNxEncoder::InitializeMXFWriter()
{
	const DNXMXF_Options_t MXFoptions{ sizeof(DNXMXF_Options_t), nullptr, AvidDNx::MXFerrorHandler };
	const DNXMXF_Rational_t FrameRate
	{
		static_cast<unsigned int>(Options.FrameRate.Numerator * 1000),
		static_cast<unsigned int>(Options.FrameRate.Denominator * 1000)
	};
	const DNXMXF_Rational_t AspectRatio = AvidDNx::AspectRatioFromResolution(Options.Width, Options.Height);

	const FFrameRate TwentyNineNineSeven = FFrameRate(30000, 1001);
	DNXMXF_TimeCodeComponent_t TimeCodeComponent = DNXMXF_TimeCodeComponent_t(
		Options.StartTimecode.Hours,
		Options.StartTimecode.Minutes,
		Options.StartTimecode.Seconds,
		Options.StartTimecode.Frames,
		Options.StartTimecode.bDropFrameFormat && (Options.FrameRate == TwentyNineNineSeven)
	);

	const DNXMXF_WriterParams_t MXFwriterParams
	{
		sizeof(DNXMXF_WriterParams_t),
		*Options.OutputFilename,
		DNXMXF_OP_1a,
		DNXMXF_WRAP_FRAME,
		FrameRate,
		VERSION_TEXT(EPIC_COMPANY_NAME),
		VERSION_TEXT(EPIC_PRODUCT_NAME),
		VERSION_STRINGIFY(ENGINE_VERSION_STRING),
		AvidDNx::ProductUID,
		nullptr,
		AspectRatio,
		0, 0,
		Options.bCompress ? DNXMXF_ESSENCE_DNXHR_HD : DNXMXF_ESSENCE_DNXUNCOMPRESSED,
		&TimeCodeComponent
	};

	if (DNXMXF_CreateWriter(&MXFoptions, &MXFwriterParams, &MXFwriter) != DNXMXF_SUCCESS)
	{
		UE_LOGF(LogAvidDNxMedia, Error, "Could not create MXF writer");
		return false;
	}

	return true;
}

bool FAvidDNxEncoder::WriteFrame_Avid(const void* InSubSampledBuffer, const int32 InSubSampledBufferSize, void* OutEncodedBuffer, const int32 InEncodedBufferSize)
{
	unsigned int CompressedFrameSize = 0;
	bool bEncodingSuccessful = false;

	const double ConversionTime = FPlatformTime::Seconds();

	if (Options.bCompress)
	{
		const int EncodeStatus = DNX_EncodeFrame(
			DNxHRencoder,
			InSubSampledBuffer,
			OutEncodedBuffer,
			InSubSampledBufferSize,
			InEncodedBufferSize,
			&CompressedFrameSize);

		if (EncodeStatus != DNX_NO_ERROR)
		{
			char ErrorString[256];
			DNX_GetErrorString(EncodeStatus, ErrorString);
			// FVideoFrameData* FramePayload = InFrame.GetPayload<FVideoFrameData>();
			// int32 FrameNumberDisplay = FramePayload->Metrics.FrameNumber;
			UE_LOGF(LogAvidDNxMedia, Error, "Unable to encode Frame: %s", ErrorString);
		}

		bEncodingSuccessful = EncodeStatus == DNX_NO_ERROR;
	}
	else
	{
		const unsigned int UncompressedBufferSize = DNXUncompressed_GetUncompressedBufSize(&DNxUncUncompressedParams);

		const DNXUncompressed_Err_t EncodeStatus = DNXUncompressed_EncodeFrame(
			DNxUncEncoder,
			&DNxUncUncompressedParams,
			&DNxUncCompressedParams,
			InSubSampledBuffer,
			UncompressedBufferSize,
			OutEncodedBuffer,
			InEncodedBufferSize,
			&CompressedFrameSize);
		bEncodingSuccessful = EncodeStatus == DNX_UNCOMPRESSED_ERR_SUCCESS; // errors will be logged through DNxUncompressedErrorHandler
	}

	if (bEncodingSuccessful)
	{
		DNXMXF_WriteFrame(MXFwriter, OutEncodedBuffer, CompressedFrameSize);
	}

	WriteEndTimeSeconds = FPlatformTime::Seconds();

	const double ConversionDeltaTimeMs = (ConversionTime - WriteStartTimeSeconds) * 1000.0;
	const double CodecDeltaTimeMs = (WriteEndTimeSeconds - ConversionTime) * 1000.0;
	const double TotalDeltaTimeMs = (WriteEndTimeSeconds - WriteStartTimeSeconds) * 1000.0;

	// FVideoFrameData* FramePayload = InFrame.GetPayload<FVideoFrameData>();
	// UE_LOGF(LogAvidDNxMedia, Verbose, "Processing Frame:%dx%d Frame:%d Conversion:%fms Codec:%fms Total:%fms",
	// 	InFrame.BufferSize.X,
	// 	InFrame.BufferSize.Y,
	// 	FramePayload->Metrics.FrameNumber,
	// 	ConversionDeltaTimeMs,
	// 	CodecDeltaTimeMs,
	// 	TotalDeltaTimeMs);
	
	return bEncodingSuccessful;
}

bool FAvidDNxEncoder::WriteFrame(const uint8* InFrameData)
{
	WriteStartTimeSeconds = FPlatformTime::Seconds();
	
	const int32 NumPixels = Options.Width * Options.Height;

	TArray<FY0CbY1Cr, TAlignedHeapAllocator<16>> SubSampledBuffer;
	SubSampledBuffer.AddUninitialized(NumPixels / 2);
	
	TArray<uint8, TAlignedHeapAllocator<16>> EncodedBuffer;
	EncodedBuffer.AddUninitialized(EncodedBufferSize);
	
	ensure(NumPixels % 2 == 0);

	const FColor* ColorData = reinterpret_cast<const FColor*>(InFrameData);
	ParallelFor(NumPixels / 2, [&SubSampledBuffer, &ColorData](const int32 SubSampledPixelIndex)
	{
		// The sub-sampled index goes from 0 -> NumPixels/2.
		// The input index goes from 0 -> NumPixels.
		
		const int32 InputIndex = SubSampledPixelIndex * 2;
		
		const FColor* PixelA = &ColorData[InputIndex];
		const FColor* PixelB = &ColorData[InputIndex + 1];

		SubSampledBuffer[SubSampledPixelIndex] = AvidDNx::RBGtoYCbCrRec709<FColor, uint8, FY0CbY1Cr>(PixelA, PixelB);
	});
	
	const char* InBuffer = reinterpret_cast<const char*>(SubSampledBuffer.GetData());
	const int32 InputBufferSize = SubSampledBuffer.Num() * 4;

	char* OutBuffer = reinterpret_cast<char*>(EncodedBuffer.GetData());
	const int32 OutBufferSize = EncodedBuffer.Num();

	return WriteFrame_Avid(InBuffer, InputBufferSize, OutBuffer, OutBufferSize);
}

bool FAvidDNxEncoder::WriteFrame_16bit(const FFloat16Color* InFrameData)
{
	const bool bIsRGB = (Options.Quality == EAvidDNxEncoderQuality::RGB444_12bit);

	if (bIsRGB)
	{
		return WriteFrame_16bit_Impl<FRGB_16bit>(InFrameData);
	}

	return WriteFrame_16bit_Impl<FY0CbY1Cr_16bit>(InFrameData);
}

const FAvidDNxEncoderOptions& FAvidDNxEncoder::GetOptions() const
{
	return Options;
}

template <typename EncodedBufferType>
bool FAvidDNxEncoder::WriteFrame_16bit_Impl(const FFloat16Color* InFrameData)
{
	WriteStartTimeSeconds = FPlatformTime::Seconds();
	
	const int32 NumPixels = Options.Width * Options.Height;
	ensure(NumPixels % 2 == 0);

	const bool bIs444 = (Options.Quality == EAvidDNxEncoderQuality::RGB444_12bit);
	const bool bIsRGB = (Options.Quality == EAvidDNxEncoderQuality::RGB444_12bit);

	// The buffer that the encoder will fill in
	TArray<uint8, TAlignedHeapAllocator<16>> EncodedBuffer;
	EncodedBuffer.AddUninitialized(EncodedBufferSize);

	// The buffer provided to the encoder. Depending on the format, this buffer may not be sub-sampled at all (eg, RGB 4:4:4).
	TArray<EncodedBufferType, TAlignedHeapAllocator<16>> SubSampledBuffer;
	SubSampledBuffer.AddUninitialized(bIs444 ? NumPixels : NumPixels / 2);

	// RGB -> Rec. 709 RGB
	if constexpr (std::is_same_v<EncodedBufferType, FRGB_16bit>)
	{
		ParallelFor(NumPixels, [&SubSampledBuffer, &InFrameData](const int32 PixelIndex)
		{
			SubSampledBuffer[PixelIndex] = AvidDNx::RGBtoRec709(&InFrameData[PixelIndex]);
		});
	}

	// RGB -> Rec. 709 YCbCr
	else
	{
		ParallelFor(NumPixels / 2, [&SubSampledBuffer, &InFrameData](const int32 SubSampledPixelIndex)
		{
			// The sub-sampled index goes from 0 -> NumPixels/2.
			// The input index goes from 0 -> NumPixels.
			
			const int32 InputIndex = SubSampledPixelIndex * 2;
			
			// Sub-sample the pixel data
			const FFloat16Color* PixelA = &InFrameData[InputIndex];
			const FFloat16Color* PixelB = &InFrameData[InputIndex + 1];

			// Convert to video range and Rec 709
			SubSampledBuffer[SubSampledPixelIndex] = AvidDNx::RBGtoYCbCrRec709<FFloat16Color, unsigned short, EncodedBufferType>(PixelA, PixelB);
		});
	}

	const char* InBuffer = reinterpret_cast<const char*>(SubSampledBuffer.GetData());
	const int32 InputBufferSize = bIsRGB ? sizeof(FRGB_16bit) * NumPixels : sizeof(FY0CbY1Cr_16bit) * NumPixels;

	void* OutBuffer = EncodedBuffer.GetData();
	const int32 OutBufferSize = EncodedBuffer.Num();

	return WriteFrame_Avid(InBuffer, InputBufferSize, OutBuffer, OutBufferSize);
}

void FAvidDNxEncoder::Finalize()
{
	if (bFinalized || !bInitialized)
	{
		return;
	}

	if (MXFwriter)
	{
		if (DNXMXF_FinishWrite(MXFwriter) != DNXMXF_SUCCESS)
		{
			UE_LOGF(LogAvidDNxMedia, Error, "Could not finish writing to MXF");
		}
		DNXMXF_DestroyWriter(MXFwriter);
	}

	if (Options.bCompress && DNxHRencoder)
	{
		DNX_DeleteEncoder(DNxHRencoder);
	}
	else if (DNxUncEncoder)
	{
		DNXUncompressed_DestroyEncoder(DNxUncEncoder);
	}

	DNX_Finalize();
	bFinalized = true;
}
