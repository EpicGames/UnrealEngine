// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvidDNxEncoderProtocol.h"

#include "AvidDNxMediaModule.h"
#include "Engine/RendererSettings.h"
#include "Misc/CoreMiscDefines.h"
#include "Runtime/Launch/Resources/Version.h"

THIRD_PARTY_INCLUDES_START
#include <AvidDNxCodec.h>
#include <dnx_mxf_sdk.h>
THIRD_PARTY_INCLUDES_END

namespace
{
	// identifier for this program which will be embedded in exported mxf files
	const wchar_t* ProductUID = L"06.9d.41.48.a0.cb.48.d4.af.19.54.da.bd.09.2a.9f";

	/**
	 * Creates one sub-sampled pixel stored in Y0CbY1Cr from two input BGRA pixels.
	 *
	 * @note The input colors will be transformed to 709 and video range at 8 bit.
	 */
	UAvidDNxEncoderProtocol::FY0CbY1Cr SubSample422(const FColor& InBgra0, const FColor& InBgra1)
	{
		const float R0 = InBgra0.R / 255.0f;
		const float G0 = InBgra0.G / 255.0f;
		const float B0 = InBgra0.B / 255.0f;

		const float R1 = InBgra1.R / 255.0f;
		const float G1 = InBgra1.G / 255.0f;
		const float B1 = InBgra1.B / 255.0f;

		// 709 conversion
		const float Yfull0 = R0 * 0.212639f + G0 * 0.7151687f + B0 * 0.0721932f;
		const float Yfull1 = R1 * 0.212639f + G1 * 0.7151687f + B1 * 0.0721932f;
		const float CbFull0 = R0 * (-0.1145922f) + G0 * (-0.3854078f) + B0 * 0.5f;
		const float CbFull1 = R1 * (-0.1145922f) + G1 * (-0.3854078f) + B1 * 0.5f;
		const float CrFull0 = R0 * 0.5f + G0 * (-0.4541555f) + B0 * (-0.04584448f);
		const float CrFull1 = R1 * 0.5f + G1 * (-0.4541555f) + B1 * (-0.04584448f);

		const float CbAvg = (CbFull0 + CbFull1) / 2.0f;
		const float CrAvg = (CrFull0 + CrFull1) / 2.0f;

		// video range conversion
		const uint8 YVideoRange0 = (uint8)FMath::RoundToInt((219 * Yfull0 + 16));
		const uint8 YVideoRange1 = (uint8)FMath::RoundToInt((219 * Yfull1 + 16));
		const uint8 CbVideoRange = (uint8)FMath::RoundToInt((224 * CbAvg + 128));
		const uint8 CrVideoRange = (uint8)FMath::RoundToInt((224 * CrAvg + 128));

		return
		{
			YVideoRange0,
			CbVideoRange,
			YVideoRange1,
			CrVideoRange
		};
	}

	int32 GCD(int32 A, int32 B)
	{
		while (B != 0)
		{
			int32 Temp = B;
			B = A % B;
			A = Temp;
		}

		return A;
	}

	DNXMXF_Rational_t AspectRatioFromResolution(const FCaptureResolution& Resolution)
	{
		const int32 Gcd = GCD(Resolution.ResX, Resolution.ResY);
		return { (unsigned int)(Resolution.ResX / Gcd), (unsigned int)(Resolution.ResY / Gcd) };
	}

	void DNxUncompressedErrorHandler(const char* Message, void* /*UserData*/)
	{
		UE_LOGF(LogAvidDNxMedia, Error, "DNxUncompressed Error: %s", Message);
	}

	void MXFerrorHandler(const char* Message, void* /*UserData*/)
	{
		UE_LOGF(LogAvidDNxMedia, Error, "Error initializing DNX MXF SDK: %s", Message);
	}
}

UAvidDNxEncoderProtocol::UAvidDNxEncoderProtocol(const FObjectInitializer& InObjInit)
	: Super(InObjInit)
	, bUncompressed(false)
	, NumberOfEncodingThreads(4)
	, bEncoderIsUncompressed(false)
	, DNxHRencoder(nullptr)
	, DNxUncEncoder(nullptr)
	, DNxUncCompressedParams()
	, DNxUncUncompressedParams()
	, MXFwriter(nullptr)
{
}

bool UAvidDNxEncoderProtocol::SetupImpl()
{
	bEncoderIsUncompressed = bUncompressed; // cached to avoid switching the encoder mid-flight
	
	if (!InitializeEncoder())
	{
		return false;
	}

	return Super::SetupImpl();
}

bool UAvidDNxEncoderProtocol::InitializeEncoder()
{
	unsigned int OutputBufferSize = 0;
	const bool bEncoderInitialized = bEncoderIsUncompressed ? InitializeUncompressedEncoder(OutputBufferSize) : InitializeHRencoder(OutputBufferSize);
	if (bEncoderInitialized)
	{
		const unsigned int Width = (unsigned int)CaptureHost->GetSettings().Resolution.ResX;
		const unsigned int Height = (unsigned int)CaptureHost->GetSettings().Resolution.ResY;
		const unsigned int SubSampledBufferSize = ((Width * Height) / 2);
		SubSampledBuffer.Init(FY0CbY1Cr{ 0 }, SubSampledBufferSize);

		ensure(OutputBufferSize > 0);
		EncodedBuffer.Init(0, OutputBufferSize);
	}

	return bEncoderInitialized && EncodedBuffer.Num() > 0;
}

bool UAvidDNxEncoderProtocol::InitializeUncompressedEncoder(unsigned int& OutEncodedBufferSize)
{
	const DNXUncompressed_Options_t Options =
	{
		sizeof(DNXUncompressed_Options_t),
		NumberOfEncodingThreads,
		nullptr,
		DNxUncompressedErrorHandler
	};

	if (DNX_UNCOMPRESSED_ERR_SUCCESS != DNXUncompressed_CreateEncoder(&Options, &DNxUncEncoder))
	{
		return false;
	}

	const unsigned int Width = (unsigned int)CaptureHost->GetSettings().Resolution.ResX;
	const unsigned int Height = (unsigned int)CaptureHost->GetSettings().Resolution.ResY;

	DNxUncUncompressedParams = DNXUncompressed_UncompressedParams_t
	{
			sizeof(DNXUncompressed_UncompressedParams_t),
			DNX_UNCOMPRESSED_CCO_YCbYCr, // color component order
			DNX_UNCOMPRESSED_CT_UCHAR, // component type
			Width,
			Height,
			0, // row bytes
			DNX_UNCOMPRESSED_FL_FULL_FRAME, // frame layout
			0 // row bytes 2
	};

	DNxUncCompressedParams = DNXUncompressed_CompressedParams_t
	{
		sizeof(DNXUncompressed_CompressedParams_t),
		false, // compress alpha
		0 // slice count for RLE if compress alpha is enabled
	};

	OutEncodedBufferSize = DNXUncompressed_GetCompressedBufSize(&DNxUncUncompressedParams, &DNxUncCompressedParams);

	return true;
}

bool UAvidDNxEncoderProtocol::InitializeHRencoder(unsigned int& OutEncodedBufferSize)
{
	const int InitResult = DNX_Initialize();
	if (InitResult != DNX_NO_ERROR)
	{
		char ErrorMessage[256];
		DNX_GetErrorString(InitResult, &ErrorMessage[0]);
		UE_LOGF(LogAvidDNxMedia, Error, "Error initializing DNX SDK: %s", ErrorMessage);
		return false;
	}

	const DNX_UncompressedParams_t UncompressedParamsHR =
	{
		sizeof(DNX_UncompressedParams_t),
		DNX_CT_UCHAR, // Component type
		DNX_CV_709, // Color volume
		DNX_CF_YCbCr, // Color format
		DNX_CCO_YCbYCr_NoA, // Component order
		DNX_BFO_Progressive, // Field order
		DNX_RGT_Display, // Raster geometry type
		0, // Interfield gap bytes
		0, // Row bytes
		// Used only for DNX_CT_SHORT_2_14
		0, // Black point
		0, // White point
		0, // ChromaExcursion
		// Used only for planar component orders
		0 // Row bytes2
	};

	const unsigned int Width = (unsigned int)CaptureHost->GetSettings().Resolution.ResX;
	const unsigned int Height = (unsigned int)CaptureHost->GetSettings().Resolution.ResY;

	const DNX_CompressedParams_t CompressedParamsHR =
	{
		sizeof(DNX_CompressedParams_t),
		Width,
		Height,
		DNX_HQ_COMPRESSION_ID,
		DNX_CV_709, // Color volume
		DNX_CF_YCbCr, // Color format
		// Parameters below are used for RI only
		DNX_SSC_422, // Sub-sampling
		8, // bit-depth, is used only for RI compression IDs
		1, // PARC
		1, // PARN
		0, // CRC-presence
		0, // VBR
		0, // Alpha-presence
		0, // Lossless alpha
		0  // Premultiplied alpha
	};

	const DNX_EncodeOperationParams_t OperationParams =
	{
		sizeof(DNX_EncodeOperationParams_t),
		NumberOfEncodingThreads
	};

	const int CreateResult = DNX_CreateEncoder(&CompressedParamsHR, &UncompressedParamsHR, &OperationParams, &DNxHRencoder);
	if (CreateResult != DNX_NO_ERROR)
	{
		char ErrorMessage[256];
		DNX_GetErrorString(CreateResult, &ErrorMessage[0]);
		UE_LOGF(LogAvidDNxMedia, Error, "Error initializing DNX Encoder: %s", ErrorMessage);
		return false;
	}

	OutEncodedBufferSize = DNX_GetCompressedBufferSize(&CompressedParamsHR);
	return true;
}

bool UAvidDNxEncoderProtocol::InitializeMXFwriter()
{
	const DNXMXF_Options_t MXFoptions{ sizeof(DNXMXF_Options_t), nullptr, MXFerrorHandler };
	const DNXMXF_Rational_t FrameRate
	{
		static_cast<unsigned int>(CaptureHost->GetCaptureFrameRate().Numerator * 1000),
		static_cast<unsigned int>(CaptureHost->GetCaptureFrameRate().Denominator * 1000)
	};
	const DNXMXF_Rational_t AspectRatio = AspectRatioFromResolution(CaptureHost->GetSettings().Resolution);
	const FString VideoOutPath = GenerateFilenameImpl(FFrameMetrics(), TEXT(".mxf"));

	const DNXMXF_WriterParams_t MXFwriterParams
	{
		sizeof(DNXMXF_WriterParams_t),
		*VideoOutPath,
		DNXMXF_OP_1a,
		DNXMXF_WRAP_FRAME,
		FrameRate,
		VERSION_TEXT(EPIC_COMPANY_NAME),
		VERSION_TEXT(EPIC_PRODUCT_NAME),
		VERSION_STRINGIFY(ENGINE_VERSION_STRING),
		ProductUID,
		nullptr,
		AspectRatio,
		0, 0,
		bEncoderIsUncompressed ? DNXMXF_ESSENCE_DNXUNCOMPRESSED : DNXMXF_ESSENCE_DNXHR_HD
	};

	if (DNXMXF_CreateWriter(&MXFoptions, &MXFwriterParams, &MXFwriter) != DNXMXF_SUCCESS)
	{
		UE_LOGF(LogAvidDNxMedia, Error, "Could not create MXF writer");
		return false;
	}

	return true;
}

struct FVideoFrameData : IFramePayload
{
	FFrameMetrics Metrics;
};

FFramePayloadPtr UAvidDNxEncoderProtocol::GetFramePayload(const FFrameMetrics& InFrameMetrics)
{
	TSharedRef<FVideoFrameData, ESPMode::ThreadSafe> FrameData = MakeShared<FVideoFrameData, ESPMode::ThreadSafe>();
	FrameData->Metrics = InFrameMetrics;
	return FrameData;
}

void UAvidDNxEncoderProtocol::ProcessFrame(FCapturedFrameData InFrame)
{
	// Defer writer creation until sequence has evaluated so that cached data for format mappings can be generated
	if (MXFwriter == nullptr)
	{
		if (!InitializeMXFwriter())
		{
			return;
		}
	}

	const double StartTime = FPlatformTime::Seconds();

	ensure(InFrame.ColorBuffer.Num() % 2 == 0);
	for (size_t InIdx = 0, OutIdx = 0; InIdx < InFrame.ColorBuffer.Num(); InIdx += 2, ++OutIdx)
	{
		SubSampledBuffer[OutIdx] = SubSample422(InFrame.ColorBuffer[InIdx], InFrame.ColorBuffer[InIdx + 1]);
	}

	const char* InBuffer = (const char*)SubSampledBuffer.GetData();
	const unsigned int InputBufferSize = SubSampledBuffer.Num() * 4;

	char* OutBuffer = (char*)EncodedBuffer.GetData();
	const int32 OutBufferSize = EncodedBuffer.Num();
	unsigned int CompressedFrameSize = 0;
	bool bEncodingSuccessful = false;

	const double ConvertionTime = FPlatformTime::Seconds();

	if (bEncoderIsUncompressed)
	{
		const unsigned int UncompressedBufferSize = DNXUncompressed_GetUncompressedBufSize(&DNxUncUncompressedParams);

		const DNXUncompressed_Err_t EncodeStatus = DNXUncompressed_EncodeFrame(
			DNxUncEncoder,
			&DNxUncUncompressedParams,
			&DNxUncCompressedParams,
			InBuffer,
			UncompressedBufferSize,
			OutBuffer,
			OutBufferSize,
			&CompressedFrameSize);
		bEncodingSuccessful = EncodeStatus == DNX_UNCOMPRESSED_ERR_SUCCESS; // errors will be logged through DNxUncompressedErrorHandler
	}
	else
	{
		const int EncodeStatus = DNX_EncodeFrame(
			DNxHRencoder,
			InBuffer,
			OutBuffer,
			InputBufferSize,
			OutBufferSize,
			&CompressedFrameSize);

		if (EncodeStatus != DNX_NO_ERROR)
		{
			char ErrorString[256];
			DNX_GetErrorString(EncodeStatus, ErrorString);
			FVideoFrameData* FramePayload = InFrame.GetPayload<FVideoFrameData>();
			int32 FrameNumberDisplay = FramePayload->Metrics.FrameNumber;
			UE_LOGF(LogAvidDNxMedia, Error, "Unable to encode Frame:%d %s", FrameNumberDisplay, ErrorString);
		}

		bEncodingSuccessful = EncodeStatus == DNX_NO_ERROR;
	}

	if (bEncodingSuccessful)
	{
		DNXMXF_WriteFrame(MXFwriter, OutBuffer, CompressedFrameSize);
	}

	const double EndTime = FPlatformTime::Seconds();

	const double ConversionDeltaTimeMs = (ConvertionTime - StartTime) * 1000.0;
	const double CodecDeltaTimeMs = (EndTime - ConvertionTime) * 1000.0;
	const double TotalDeltaTimeMs = (EndTime - StartTime) * 1000.0;

	FVideoFrameData* FramePayload = InFrame.GetPayload<FVideoFrameData>();
	UE_LOGF(LogAvidDNxMedia, Verbose, "Processing Frame:%dx%d Frame:%d Conversion:%fms Codec:%fms Total:%fms",
		InFrame.BufferSize.X,
		InFrame.BufferSize.Y,
		FramePayload->Metrics.FrameNumber,
		ConversionDeltaTimeMs,
		CodecDeltaTimeMs,
		TotalDeltaTimeMs);
}

void UAvidDNxEncoderProtocol::DestroyEncoder()
{
	if (bEncoderIsUncompressed)
	{
		if (DNxUncEncoder)
		{
			DNXUncompressed_DestroyEncoder(DNxUncEncoder);
		}
	}
	else
	{
		if (DNxHRencoder)
		{
			DNX_DeleteEncoder(DNxHRencoder);
		}
		DNX_Finalize();
	}
}

void UAvidDNxEncoderProtocol::DestroyMXFwriter()
{
	if (MXFwriter)
	{
		if (DNXMXF_FinishWrite(MXFwriter) != DNXMXF_SUCCESS)
		{
			UE_LOGF(LogAvidDNxMedia, Error, "Could not finish writing to MXF");
		}
		DNXMXF_DestroyWriter(MXFwriter);
	}
}

void UAvidDNxEncoderProtocol::FinalizeImpl()
{
	Super::FinalizeImpl();

	DestroyMXFwriter();
	DestroyEncoder();
}
