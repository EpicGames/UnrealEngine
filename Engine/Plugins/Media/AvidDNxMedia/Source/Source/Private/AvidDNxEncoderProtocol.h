// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Protocols/FrameGrabberProtocol.h"

THIRD_PARTY_INCLUDES_START
#include <dnx_uncompressed_sdk.h>
THIRD_PARTY_INCLUDES_END

#include "AvidDNxEncoderProtocol.generated.h"

struct _DNX_Encoder;
using DNX_Encoder = struct _DNX_Encoder*;
using DNXMXF_Writer = struct DNXMXF_Writer;

UCLASS(meta = (DisplayName = "Avid DNx Encoder (mxf)", CommandLineID = "AvidDNx") )
class AVIDDNXMEDIA_API UAvidDNxEncoderProtocol : public UFrameGrabberProtocol
{
public:
	GENERATED_BODY()

	UAvidDNxEncoderProtocol(const FObjectInitializer& InObjInit);

public:
	virtual bool SetupImpl() override;
	virtual void FinalizeImpl() override;
	virtual FFramePayloadPtr GetFramePayload(const FFrameMetrics& InFrameMetrics) override;
	virtual void ProcessFrame(FCapturedFrameData InFrame) override;

public:
	UPROPERTY(config, EditAnywhere, Category = "Avid DNX Settings")
	bool bUncompressed;

	UPROPERTY(config, EditAnywhere, Category = "Avid DNX Settings", meta = (ClampMin = "1", UIMin = "1", ClampMax = "64", UIMax = "64"))
	uint8 NumberOfEncodingThreads;

public:
	struct FY0CbY1Cr
	{
		uint8 Y0, Cb, Y1, Cr;
	};

private:
	bool InitializeEncoder();
	bool InitializeUncompressedEncoder(unsigned int& OutEncodedBufferSize);
	bool InitializeHRencoder(unsigned int& OutEncodedBufferSize);
	bool InitializeMXFwriter();

	void DestroyEncoder();
	void DestroyMXFwriter();

private:
	bool bEncoderIsUncompressed; // cached from bUncompressed

	DNX_Encoder DNxHRencoder;

	DNXUncompressed_Encoder* DNxUncEncoder;
	DNXUncompressed_CompressedParams_t DNxUncCompressedParams;
	DNXUncompressed_UncompressedParams_t DNxUncUncompressedParams;

	DNXMXF_Writer* MXFwriter;

	TArray<FY0CbY1Cr, TAlignedHeapAllocator<16>> SubSampledBuffer;
	TArray<uint8, TAlignedHeapAllocator<16>> EncodedBuffer;
};
