// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvidDNxMediaModule.h"

#include "HAL/Platform.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "AvidDNx"

DEFINE_LOG_CATEGORY(LogAvidDNxMedia);

class FAvidDNxMediaModule : public IModuleInterface
{
	virtual void StartupModule() override
	{
		LibHandleDNx = LoadDll(TEXT("DNxHR.dll"));
		LibHandleMXF = LoadDll(TEXT("DNxMXF-dynamic.dll"));
		LibHandleDNxUncompressed = LoadDll(TEXT("DNxUncompressedSDK.dll"));

		// Add exemption to FName::NameToDisplayString formatting to ensure "DNxHR" is displayed without a space
		FName::AddNameToDisplayStringExemption(TEXT("DNxHR"));
	}

	virtual void ShutdownModule() override
	{
		FreeDll(LibHandleDNx);
		FreeDll(LibHandleMXF);
		FreeDll(LibHandleDNxUncompressed);
	}

	void* LoadDll(const TCHAR* InDllName)
	{
		void* DllHandle = nullptr;

// The DLLs are copied across into the Engine/Binaries/Win64 folder now when building.
// There is no need to construct a path and load from there any more.
#if 0
		// determine directory paths
		FString DllPath = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("AvidDNxMedia"))->GetBaseDir(), TEXT("/Binaries/ThirdParty/Win64"));
		FPlatformProcess::PushDllDirectory(*DllPath);
		DllPath = FPaths::Combine(DllPath, InDllName);

		if (!FPaths::FileExists(DllPath))
		{
			UE_LOGF(LogAvidDNxMedia, Error, "Failed to find the binary folder for %ls. Plug-in will not be functional.", InDllName);
			return nullptr;
		}

		DllHandle = FPlatformProcess::GetDllHandle(*DllPath);
		if (!DllHandle)
		{
			UE_LOGF(LogAvidDNxMedia, Error, "Failed to load required library %ls. Plug-in will not be functional.", *DllPath);
		}
#else
		DllHandle = FPlatformProcess::GetDllHandle(InDllName);
		if (!DllHandle)
		{
			UE_LOGF(LogAvidDNxMedia, Error, "Failed to load required library %ls. Plug-in will not be functional.", InDllName);
		}
#endif

		return DllHandle;
	}

	void FreeDll(void* OutHandle)
	{
		if (OutHandle)
		{
			FPlatformProcess::FreeDllHandle(OutHandle);
			OutHandle = nullptr;
		}
	}

	// Codec could still be in use
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

private:
	void* LibHandleDNx = nullptr;
	void* LibHandleMXF = nullptr;
	void* LibHandleDNxUncompressed = nullptr;
};

IMPLEMENT_MODULE(FAvidDNxMediaModule, AvidDNxMedia)

#undef LOCTEXT_NAMESPACE
