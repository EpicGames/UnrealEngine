// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphAvidDNxHRNode.h"

#include "ImageWriteTask.h"
#include "MoviePipelineImageQuantization.h"
#include "MoviePipelineTelemetry.h"
#include "AvidDNxEncoder/AvidDNxEncoder.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieGraphOCIOHelper.h"
#include "Graph/Nodes/MovieGraphBurnInNode.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Styling/AppStyle.h"

UMovieGraphAvidDNxHRNode::UMovieGraphAvidDNxHRNode()
{
}

#if WITH_EDITOR
FText UMovieGraphAvidDNxHRNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText AvidDNxHRNodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_AvidDNxHR", "Avid DNxHR Movie");
	return AvidDNxHRNodeName;
}

FText UMovieGraphAvidDNxHRNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "AvidDNxHRNode_Category", "Output Type");
}

FText UMovieGraphAvidDNxHRNode::GetKeywords() const
{
	static const FText Keywords = NSLOCTEXT("MovieGraphNodes", "AvidDNxHRGraphNode_Keywords", "avid dnxhr mxf mov movie video");
	return Keywords;
}

FLinearColor UMovieGraphAvidDNxHRNode::GetNodeTitleColor() const
{
	static const FLinearColor AvidDNxHRNodeColor = FLinearColor(0.047f, 0.654f, 0.537f);
	return AvidDNxHRNodeColor;
}

FSlateIcon UMovieGraphAvidDNxHRNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon AvidDNxHRIcon = FSlateIcon(FName("MovieRenderPipelineStyle"), "MovieRenderPipeline.Graph.Icon.RenderMovieFile");

	OutColor = FLinearColor::White;
	return AvidDNxHRIcon;
}

FText UMovieGraphAvidDNxHRNode::GetBasicConfigShortDisplayName() const
{
	return NSLOCTEXT("MovieGraphNodes", "BasicConfigShortName_AVID", "AVID");
}
#endif // WITH_EDITOR

TUniquePtr<MovieRenderGraph::IVideoCodecWriter> UMovieGraphAvidDNxHRNode::Initialize_GameThread(const FMovieGraphVideoNodeInitializationContext& InInitializationContext)
{
	bool bIncludeCDOs = true;
	constexpr bool bExactMatch = true;
	UMovieGraphGlobalOutputSettingNode* OutputSetting =
		InInitializationContext.EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);

	bIncludeCDOs = false;
	const UMovieGraphAvidDNxHRNode* EvaluatedNode = Cast<UMovieGraphAvidDNxHRNode>(
		InInitializationContext.EvaluatedConfig->GetSettingForBranch(GetClass(), InInitializationContext.PassData->Key.RootBranchName, bIncludeCDOs, bExactMatch));
	checkf(EvaluatedNode, TEXT("Avid DNxHR node could not be found in the graph in branch [%s]."), *InInitializationContext.PassData->Key.RootBranchName.ToString());
	
	const FFrameRate SourceFrameRate = InInitializationContext.Pipeline->GetDataSourceInstance()->GetDisplayRate();
	const FFrameRate EffectiveFrameRate = UMovieGraphBlueprintLibrary::GetEffectiveFrameRate(OutputSetting, SourceFrameRate);

	// Determine the timecode that the movie should be started at
	FTimecode StartTimecode;
	if (OutputSetting->bOverride_CustomTimecodeStart)
	{
		const int32 OutputFrameNumber = InInitializationContext.TraversalContext->Time.OutputFrameNumber;

		// When using a custom timecode start, just use the root-level frame number (relative to zero) offset by the custom timecode start
		StartTimecode = FTimecode::FromFrameNumber(
			OutputFrameNumber + OutputSetting->CustomTimecodeStart.ToFrameNumber(EffectiveFrameRate).Value,
			EffectiveFrameRate,
			OutputSetting->bDropFrameTimecode);
	}
	else
	{
		// This is the frame number on the global time, can have overlaps (between encoders) or repeats when using handle frames/slowmo.
		StartTimecode = InInitializationContext.TraversalContext->Time.RootTimeCode;
	}
	
	FAvidDNxEncoderOptions Options;
	Options.OutputFilename = InInitializationContext.FileName;
	Options.Width = InInitializationContext.Resolution.X;
	Options.Height = InInitializationContext.Resolution.Y;
	Options.Quality = EvaluatedNode->Quality;
	Options.FrameRate = EffectiveFrameRate;
	Options.bCompress = true;
	Options.NumberOfEncodingThreads = 4;
	Options.bDropFrameTimecode = OutputSetting->bDropFrameTimecode;
	Options.StartTimecode = StartTimecode;

	// If OCIO is enabled, don't do additional color conversion. RGB444 12-bit is never converted to sRGB.
	if (EvaluatedNode->Quality == EAvidDNxEncoderQuality::RGB444_12bit)
	{
		Options.bConvertToSrgb = false;
	}
	else
	{
		Options.bConvertToSrgb = !(EvaluatedNode->bOverride_OCIOConfiguration && EvaluatedNode->OCIOConfiguration.bIsEnabled && InInitializationContext.bAllowOCIO);
	}
	
	TUniquePtr<FAvidWriter> NewWriter = MakeUnique<FAvidWriter>();
	NewWriter->Writer = MakeUnique<FAvidDNxEncoder>(Options);

	CachedPipeline = InInitializationContext.Pipeline;
	
	return NewWriter;
}

bool UMovieGraphAvidDNxHRNode::Initialize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter)
{
	const FAvidWriter* CodecWriter = static_cast<FAvidWriter*>(InWriter);
	if(!CodecWriter->Writer->Initialize())
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Failed to initialize Avid DNxHR writer.");
		return false;
	}
	
	return true;
}

void UMovieGraphAvidDNxHRNode::WriteFrame_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<FMovieGraphPassData>&& InCompositePasses, TObjectPtr<UMovieGraphEvaluatedConfig> InEvaluatedConfig, const FString& InBranchName)
{
	const FAvidWriter* CodecWriter = static_cast<FAvidWriter*>(InWriter);
	
	const UE::MovieGraph::FMovieGraphSampleState* Payload = InPixelData->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();

	constexpr bool bIncludeCDOs = false;
	constexpr bool bExactMatch = true;
	const FName BranchName = Payload->TraversalContext.RenderDataIdentifier.RootBranchName;
	const UMovieGraphAvidDNxHRNode* EvaluatedNode = Cast<UMovieGraphAvidDNxHRNode>(
		InEvaluatedConfig->GetSettingForBranch(GetClass(), BranchName, bIncludeCDOs, bExactMatch));
	checkf(EvaluatedNode, TEXT("Avid DNxHR node could not be found in the graph in branch [%s]."), *BranchName.ToString());

	TArray<FPixelPreProcessor> PixelPreProcessors;

	// Run OCIO before quantizing. For highest accuracy, OCIO should run on the non-quantized pixel data.
#if WITH_OCIO
	if (FMovieGraphOCIOHelper::GenerateOcioPixelPreProcessor(Payload, CachedPipeline.Get(), InEvaluatedConfig, EvaluatedNode->OCIOConfiguration, EvaluatedNode->OCIOContext, PixelPreProcessors))
	{
		PixelPreProcessors.Pop()(InPixelData);
	}
#endif

	// Quantize our 16-bit float data to 8/16-bit and apply sRGB if needed
	const int32 BitDepth = ((EvaluatedNode->Quality == EAvidDNxEncoderQuality::HQX_10bit) || (EvaluatedNode->Quality == EAvidDNxEncoderQuality::RGB444_12bit)) ? 16 : 8;
	const TUniquePtr<FImagePixelData> QuantizedPixelData = UE::MoviePipeline::QuantizeImagePixelDataToBitDepth(InPixelData, BitDepth, nullptr, CodecWriter->Writer->GetOptions().bConvertToSrgb);

	// Do a quick composite of renders/burn-ins.
	for (const FMovieGraphPassData& CompositePass : InCompositePasses)
	{
		// We don't need to copy the data here (even though it's being passed to a async system) because we already made a unique copy of the
		// burn in/widget data when we decided to composite it.
		switch (QuantizedPixelData->GetType())
		{
		case EImagePixelType::Color:
			PixelPreProcessors.Add(TAsyncCompositeImage<FColor>(CompositePass.Value->MoveImageDataToNew()));
			break;
		case EImagePixelType::Float16:
			PixelPreProcessors.Add(TAsyncCompositeImage<FFloat16Color>(CompositePass.Value->MoveImageDataToNew()));
			break;
		case EImagePixelType::Float32:
			PixelPreProcessors.Add(TAsyncCompositeImage<FLinearColor>(CompositePass.Value->MoveImageDataToNew()));
			break;
		}
	}

	// This is done on the current thread for simplicity but the composite itself is parallelized.
	FImagePixelData* PixelData = QuantizedPixelData.Get();
	for (const FPixelPreProcessor& PreProcessor : PixelPreProcessors)
	{
		// PreProcessors are assumed to be valid.
		PreProcessor(PixelData);
	}

	const void* Data = nullptr;
	int64 DataSize;
	QuantizedPixelData->GetRawData(Data, DataSize);

	if (BitDepth == 8)
	{
		CodecWriter->Writer->WriteFrame((uint8*)Data);
	}
	else
	{
		CodecWriter->Writer->WriteFrame_16bit((FFloat16Color*)Data);
	}
}

void UMovieGraphAvidDNxHRNode::BeginFinalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter)
{
	return;
}

void UMovieGraphAvidDNxHRNode::Finalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter)
{
	// Write to disk
	const FAvidWriter* CodecWriter = static_cast<FAvidWriter*>(InWriter);
	CodecWriter->Writer->Finalize();
}

const TCHAR* UMovieGraphAvidDNxHRNode::GetFilenameExtension() const
{
	// TODO: This should return "mov" when MOV is supported and selected
	return TEXT("mxf");
}

bool UMovieGraphAvidDNxHRNode::IsAudioSupported() const
{
	// The current Avid DNxHR SDK does not support audio encoding so we don't write audio to the container.
	return false;
}

void UMovieGraphAvidDNxHRNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesAvid = true;
}

void UMovieGraphAvidDNxHRNode::InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes)
{
	if (bEnableBurnIn)
	{
		UMovieGraphOutputBurnInNode* BurnInNode = NewObject<UMovieGraphOutputBurnInNode>(InEvaluatedConfig);
		BurnInNode->bOverride_OutputName = true;
		BurnInNode->bOverride_FileNameFormat = true;
		BurnInNode->bOverride_OutputRestriction = true;
		BurnInNode->bOverride_LayerNameFormat = true;
		BurnInNode->bOverride_BurnInClass = true;
		BurnInNode->bOverride_bCompositeOntoFinalImage = true;
		BurnInNode->OutputName = TEXT("Avid");
		BurnInNode->FileNameFormat = BurnInFileNameFormat;
		BurnInNode->OutputRestriction = FSoftClassPath(GetClass());
		BurnInNode->LayerNameFormat = TEXT("");	// Unused
		BurnInNode->BurnInClass = BurnInClass;
		BurnInNode->bCompositeOntoFinalImage = bCompositeOntoFinalImage;

		OutInjectedNodes.Add(BurnInNode);
	}
}
