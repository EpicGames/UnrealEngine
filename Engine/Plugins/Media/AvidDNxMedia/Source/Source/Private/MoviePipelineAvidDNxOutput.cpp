// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineAvidDNxOutput.h"
#include "MoviePipeline.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineTelemetry.h"
#include "ImagePixelData.h"
#include "MoviePipelineImageQuantization.h"
#include "SampleBuffer.h"
#include "MovieRenderPipelineCoreModule.h"
#include "ImageWriteTask.h"
#include "MovieRenderPipelineDataTypes.h"

// For logs
#include "MovieRenderPipelineCoreModule.h"

TUniquePtr<MovieRenderPipeline::IVideoCodecWriter> UMoviePipelineAvidDNxOutput::Initialize_GameThread(const FString& InFileName, FIntPoint InResolution, EImagePixelType InPixelType, ERGBFormat InPixelFormat, uint8 InBitDepth, uint8 InNumChannels)
{
	UMoviePipelineOutputSetting* OutputSettings = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	if (!OutputSettings)
	{
		return nullptr;
	}
		
	 
	FAvidDNxEncoderOptions Options;
	Options.OutputFilename = InFileName;
	Options.Width = InResolution.X;
	Options.Height = InResolution.Y;
	Options.FrameRate = GetPipeline()->GetPipelinePrimaryConfig()->GetEffectiveFrameRate(GetPipeline()->GetTargetSequence());
	Options.bCompress = bUseCompression;
	Options.NumberOfEncodingThreads = NumberOfEncodingThreads;

	TUniquePtr<FAvidDNxEncoder> Encoder = MakeUnique<FAvidDNxEncoder>(Options);
	
	TUniquePtr<FAvidWriter> OutWriter = MakeUnique<FAvidWriter>();
	OutWriter->Writer = MoveTemp(Encoder);
	OutWriter->FileName = InFileName;
	
	return OutWriter;
}

bool UMoviePipelineAvidDNxOutput::Initialize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter)
{
	FAvidWriter* CodecWriter = static_cast<FAvidWriter*>(InWriter);
	if(!CodecWriter->Writer->Initialize())
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Failed to initialize Avid DNxHD/Avid DNxHR Writer.");
		return false;
	}
	return true;
}

void UMoviePipelineAvidDNxOutput::WriteFrame_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<MoviePipeline::FCompositePassInfo>&& InCompositePasses)
{
	FAvidWriter* CodecWriter = static_cast<FAvidWriter*>(InWriter);
	
	// Quantize our 16 bit float data to 8 bit and apply sRGB
	TUniquePtr<FImagePixelData> QuantizedPixelData = UE::MoviePipeline::QuantizeImagePixelDataToBitDepth(InPixelData, 8, nullptr, InWriter->bConvertToSrgb);

	// Do a quick composite of renders/burn-ins.
	TArray<FPixelPreProcessor> PixelPreProcessors;
	for (const MoviePipeline::FCompositePassInfo& CompositePass : InCompositePasses)
	{
		// We don't need to copy the data here (even though it's being passed to a async system) because we already made a unique copy of the
		// burn in/widget data when we decided to composite it.
		switch (QuantizedPixelData->GetType())
		{
		case EImagePixelType::Color:
			PixelPreProcessors.Add(TAsyncCompositeImage<FColor>(CompositePass.PixelData->MoveImageDataToNew()));
			break;
		case EImagePixelType::Float16:
			PixelPreProcessors.Add(TAsyncCompositeImage<FFloat16Color>(CompositePass.PixelData->MoveImageDataToNew()));
			break;
		case EImagePixelType::Float32:
			PixelPreProcessors.Add(TAsyncCompositeImage<FLinearColor>(CompositePass.PixelData->MoveImageDataToNew()));
			break;
		}
	}

	// This is done on the main thread for simplicity but the composite itself is parallaleized.
	FImagePixelData* PixelData = QuantizedPixelData.Get();
	for (const FPixelPreProcessor& PreProcessor : PixelPreProcessors)
	{
		// PreProcessors are assumed to be valid.
		PreProcessor(PixelData);
	}

	const void* Data = nullptr;
	int64 DataSize;
	QuantizedPixelData->GetRawData(Data, DataSize);

	CodecWriter->Writer->WriteFrame((uint8*)Data);
}

void UMoviePipelineAvidDNxOutput::BeginFinalize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter)
{
	// The current AvidDNx SDK does not support audio encoding so we don't write audio to the container.
	return;
}

void UMoviePipelineAvidDNxOutput::Finalize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter)
{
	// Commit this to disk.
	FAvidWriter* CodecWriter = static_cast<FAvidWriter*>(InWriter);
	CodecWriter->Writer->Finalize();
}

void UMoviePipelineAvidDNxOutput::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesAvid = true;
}
