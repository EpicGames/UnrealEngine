// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePipelineVideoOutputBase.h"
#include "AvidDNxEncoder/AvidDNxEncoder.h"
#include "MoviePipelineAvidDNxOutput.generated.h"

// Forward Declare
struct FMoviePipelineShotRenderTelemetry;

UCLASS(BlueprintType)
class UMoviePipelineAvidDNxOutput : public UMoviePipelineVideoOutputBase
{
	GENERATED_BODY()

		UMoviePipelineAvidDNxOutput()
		: UMoviePipelineVideoOutputBase()
		, bUseCompression(true)
		, NumberOfEncodingThreads(4)
	{
	}

protected:
	// UMoviePipelineVideoOutputBase Interface
	virtual TUniquePtr<MovieRenderPipeline::IVideoCodecWriter> Initialize_GameThread(const FString& InFileName, FIntPoint InResolution, EImagePixelType InPixelType, ERGBFormat InPixelFormat, uint8 InBitDepth, uint8 InNumChannels) override;
	virtual bool Initialize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter) override;
	virtual void WriteFrame_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<MoviePipeline::FCompositePassInfo>&& InCompositePasses) override;
	virtual void BeginFinalize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter);
	virtual void Finalize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter);
	virtual const TCHAR* GetFilenameExtension() const override { return TEXT("mxf"); }
	virtual bool IsAudioSupported() const { return false; }
	// ~UMoviePipelineVideoOutputBase Interface

	// UMoviePipelineOutputBase Interface
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "AvidDNx_DisplayName", "Avid DNx [8bit]"); }
#endif
	// ~UMoviePipelineOutputBase Interface

	// UMoviePipelineSetting Interface
	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	// ~UMoviePipelineSetting Interface

public:
	/** Should we use a lossy compression for the output? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bUseCompression;

	/** How many threads should the AvidDNx Encoders use to encode frames? */
	UPROPERTY(EditAnywhere, meta=(UIMin=1, ClampMin=1), BlueprintReadWrite, Category = "Settings")
	int32 NumberOfEncodingThreads;

protected:
	struct FAvidWriter : public MovieRenderPipeline::IVideoCodecWriter
	{
		TUniquePtr<FAvidDNxEncoder> Writer;
	};
};