// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"

//THIRDPARTY_INCLUDES_START
#include <AvidDNxCodec.h>
#include <dnx_uncompressed_sdk.h>
//THIRDPARTY_INCLUDES_END

#include "AvidDNxEncoder.generated.h"

// Forward Declares
struct _DNX_Encoder;
using DNX_Encoder = struct _DNX_Encoder*;
using DNXMXF_Writer = struct DNXMXF_Writer;

/** Quality settings available for the Avid DNx encoder. */
UENUM()
enum class EAvidDNxEncoderQuality
{
	/** Cinema quality, 12-bit 4:4:4 RGB. */
	RGB444_12bit	= DNX_444_COMPRESSION_ID	UMETA(DisplayName = "DNxHR RGB 444 12-bit"),

	/** High quality extended, 10-bit 4:2:2 YCbCr. */
	HQX_10bit		= DNX_HQX_COMPRESSION_ID	UMETA(DisplayName = "DNxHR HQX 10-bit"),

	/** High quality, 8-bit 4:2:2 YCbCr. */
	HQ_8bit			= DNX_HQ_COMPRESSION_ID		UMETA(DisplayName = "DNxHR HQ 8-bit"),

	/** Standard quality, 8-bit 4:2:2 YCbCr. */
	SQ_8bit			= DNX_SQ_COMPRESSION_ID		UMETA(DisplayName = "DNxHR SQ 8-bit"),

	/** Low bandwidth, 8-bit 4:2:2 YCbCr. */
	LB_8bit			= DNX_LB_COMPRESSION_ID		UMETA(DisplayName = "DNxHR LB 8-bit")
};

/** 
* Options to initialize the AvidDNx encoder with. Choosing compression will choose the AvidDNxHR HD compression.
*/
struct FAvidDNxEncoderOptions
{
	FAvidDNxEncoderOptions()
		: OutputFilename()
		, Width(0)
		, Height(0)
		, Quality(EAvidDNxEncoderQuality::HQ_8bit)
		, FrameRate(30, 1)
		, bCompress(true)
		, NumberOfEncodingThreads(0)
		, bDropFrameTimecode(false)
	{}

	/** The absolute path on disk to try and save the video file to. */
	FString OutputFilename;

	/** The width of the video file. */
	uint32 Width;

	/** The height of the video file. */
	uint32 Height;

	/** The quality setting that the encoder should use. */
	EAvidDNxEncoderQuality Quality;
	
	/** Whether the data should be converted to sRGB before being sent to the encoder. Should not be used when OCIO is active. */
	bool bConvertToSrgb;

	/** Frame Rate of the output video. */
	FFrameRate FrameRate;
	
	/** Should we use a compression codec or not */
	bool bCompress;
	
	/** Number of Encoding Threads. Must be at least 1. */
	uint32 NumberOfEncodingThreads;

	/** If true, timecode track will use drop frame notation for the 29.97 frame rate. */
	bool bDropFrameTimecode;

	/** The timecode to start the movie at. */
	FTimecode StartTimecode;
};

/**
* Encoder class that takes sRGB 8-bit RGBA data and encodes it to AvidDNxHR or AvidDNxHD before placing it in an mxf container.
* The mxf container writer currently implemented does not support audio, so audio writing APIs have been omitted from this encoder.
*/
class AVIDDNXMEDIA_API FAvidDNxEncoder
{
public:
	FAvidDNxEncoder(const FAvidDNxEncoderOptions& InOptions);
	~FAvidDNxEncoder();

	/** Call to initialize the encoder. This must be done before attempting to write data to it. */
	bool Initialize();

	/** Finalize the video file and finish writing it to disk. Called by the destructor if not automatically called. */
	void Finalize();

	/** Appends a new frame onto the output file (8-bit). */
	bool WriteFrame(const uint8* InFrameData);

	/** Appends a new frame onto the output file (16-bit). */
	bool WriteFrame_16bit(const FFloat16Color* InFrameData);

	/** Gets the options that the encoder was initialized with. */
	const FAvidDNxEncoderOptions& GetOptions() const;

private:
	bool InitializeCompressedEncoder();
	bool InitializeUncompressedEncoder();
	bool InitializeMXFWriter();

	/** Ask the AVID SDK to encode and write the buffer. */
	bool WriteFrame_Avid(const void* InSubSampledBuffer, int32 InSubSampledBufferSize, void* OutEncodedBuffer, int32 InEncodedBufferSize);

	/** Templated implementation of WriteFrame_16bit(). EncodedBufferType supports FRGB_16bit and FY0CbY1Cr_16bit. */
	template <typename EncodedBufferType>
	bool WriteFrame_16bit_Impl(const FFloat16Color* InFrameData);

public:
	/** 8-bit sub-sampled YCbCr color. */
	struct FY0CbY1Cr
	{
		uint8 Y0, Cb, Y1, Cr;
	};

	/** 16-bit sub-sampled YCbCr color. */
	struct FY0CbY1Cr_16bit
	{
		unsigned short Y0, Cb, Y1, Cr;
	};

	/** 16-bit RGB color. */
	struct FRGB_16bit
	{
		unsigned short R, G, B;
	};

private:
	FAvidDNxEncoderOptions Options;
	bool bInitialized;
	bool bFinalized;

	/** When encoding and writing a frame started. */
	double WriteStartTimeSeconds;

	/** When encoding and writing a frame finished. */
	double WriteEndTimeSeconds;
	
	/** How big each video sample is after compression based on given settings. */
	int32 EncodedBufferSize;
	
	/** Encoder used for compressed output. */
	DNX_Encoder DNxHRencoder;

	/** Encoder used for uncompressed output. */
	DNXUncompressed_Encoder* DNxUncEncoder;

	DNXUncompressed_CompressedParams_t DNxUncCompressedParams;
	DNXUncompressed_UncompressedParams_t DNxUncUncompressedParams;

	DNXMXF_Writer* MXFwriter;
};